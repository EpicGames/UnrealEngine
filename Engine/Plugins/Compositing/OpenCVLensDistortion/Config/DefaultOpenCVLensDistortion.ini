[CoreRedirects]

;
; 4.20 Redirects
;

+PackageRedirects=(OldName="/Script/MixedRealityCaptureCalibration", NewName="/Script/OpenCVLensCalibration")

+StructRedirects=(OldName="MrcLensDistortion", NewName="/Script/OpenCVLensDistortion.OpenCVLensDistortionParameters")
+ClassRedirects=(OldName="MrcOpenCVCalibrator", NewName="/Script/OpenCVLensCalibration.OpenCVLensCalibrator")
+PropertyRedirects=(OldName="FeedRenderTarget.TexRT", NewName="FeedRenderTarget.TextureRenderTarget")
+PropertyRedirects=(OldName="CalculateLensParameters.LensDistortion", NewName="CalculateLensParameters.LensDistortionParameters")