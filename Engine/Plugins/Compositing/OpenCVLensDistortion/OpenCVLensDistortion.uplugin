{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "OpenCV Lens Distortion",
	"Description": "Plugin to handle camera calibration and lens distortion/undistortion displacement map generation using OpenCV.",
	"Category": "Compositing",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "OpenCV",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "OpenCVLensCalibration",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			]
		},
		{
			"Name": "OpenCVLensDistortion",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			]
		}
	]
}