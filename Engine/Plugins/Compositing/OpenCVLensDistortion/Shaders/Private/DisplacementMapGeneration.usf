// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DisplacementMapGenerator.usf: Generate lens distortion and undistortion
	UV displacement map into a render target from precomputed undistortion
	UV displacement map.

	The pixel shader directly computes the distort viewport UV to undistort
	viewport UV displacment using Sv_Position and the pre-computed map and
	store them into the red and green channels.

	However to avoid resolving with a ferrari method, or doing a newton method
	on the GPU to compute the undistort viewport UV to distort viewport UV
	displacement, this couple of shaders works as follow: The vertex shader
	undistort the grid's vertices, and passdown to the pixel shader the viewport
	UV of where they should have been on screen without undistortion. The pixel
	shader can then generate the undistort viewport UV to distort viewport UV
	displacement by just subtracting the pixel's viewport UV.
=============================================================================*/

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"


// Size of the pixels in the viewport UV coordinates.
float2 PixelUVSize;

// Pre computed undistort displacement map
Texture2D UndistortDisplacementMap;

//Sampler to use with pre computed map
SamplerState BilinearClampedSampler;


// Flip UV's y component.
float2 FlipUV(float2 UV)
{
	return float2(UV.x, 1 - UV.y);
}

float2 ComputeGridVertexUV(uint GlobalVertexId)
{
	// Compute the cell index.
	uint GridCellIndex = GlobalVertexId / 6;

	// Compute row and column id of the cell within the grid.
	uint GridColumnId = GridCellIndex / GRID_SUBDIVISION_Y;
	uint GridRowId = GridCellIndex - GridColumnId * GRID_SUBDIVISION_Y;

	// Compute the vertex id within a 2 triangles grid cell.
	uint VertexId = GlobalVertexId - GridCellIndex * 6;

	// Compute the bottom left originated UV coordinate of the triangle's vertex within the cell.
	float2 CellVertexUV = float2(0x1 & ((VertexId + 1) / 3), VertexId & 0x1);

	// Compute the top left originated UV of the vertex within the grid.
	float2 GridInvSize = 1.f / float2(GRID_SUBDIVISION_X, GRID_SUBDIVISION_Y);
	float2 GridVertexUV = FlipUV(GridInvSize * (CellVertexUV + float2(GridColumnId, GridRowId)));

	return GridVertexUV;
}

void MainVS(in uint GlobalVertexId : SV_VertexID
		  , out float2 OutVertexDistortedViewportUV : TEXCOORD0
		  , out float4 OutPosition : SV_POSITION)
{
	//Get top left originated UV of the vertex from the distorted space
	float2 GridVertexUV = ComputeGridVertexUV(GlobalVertexId);

	// Fetch the undistort UV from precomputed map
	float2 UndistortUV = UndistortDisplacementMap.SampleLevel(BilinearClampedSampler, GridVertexUV, 0).xy + GridVertexUV;

	// Output vertex position.
	OutPosition = float4(FlipUV(UndistortUV) * 2 - 1, 0, 1);

	// Output top left originated UV of the vertex.
	OutVertexDistortedViewportUV = GridVertexUV;
}


void MainPS(in noperspective float2 VertexDistortedViewportUV : TEXCOORD0
		  , in float4 SvPosition : SV_POSITION
		  , out float4 OutColor : SV_Target0)
{
	// Compute the pixel's top left originated UV.
	float2 ViewportUV = SvPosition.xy * PixelUVSize;

	// Fetch the undistort UV from precomputed map
	float2 UndistortUV = UndistortDisplacementMap.SampleLevel(BilinearClampedSampler, ViewportUV, 0).xy + ViewportUV;

	//Compute both displacement UVs
	float2 DistortUVtoUndistortUV = UndistortUV - ViewportUV;
	float2 UndistortUVtoDistortUV = VertexDistortedViewportUV - ViewportUV;

	// Output displacement channels.
	OutColor = float4(DistortUVtoUndistortUV, UndistortUVtoDistortUV);
}
