// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOpenCVLensCalibrationModule.h"



DEFINE_LOG_CATEGORY(LogOpenCVLensCalibration)

//////////////////////////////////////////////////////////////////////////
// FOpenCVLensCalibrationModule
class FOpenCVLensCalibrationModule : public IOpenCVLensCalibrationModule
{

};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FOpenCVLensCalibrationModule, OpenCVLensCalibration)
