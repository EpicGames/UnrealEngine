// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenCVLensCalibrator.h"

#include "Engine/TextureRenderTarget2D.h"
#include "IOpenCVLensCalibrationModule.h"
#include "TextureResource.h"

#if WITH_OPENCV


#include "opencv2/calib3d.hpp"
#include "opencv2/imgproc.hpp"


#endif


UOpenCVLensCalibrator* UOpenCVLensCalibrator::CreateCalibrator(int32 InBoardWidth /*= 7*/, int32 InBoardHeight /*= 5*/, float InSquareSize /*= 3.f*/, bool bInUseFisheyeModel /*= false*/)
{
#if WITH_OPENCV == 0
	UE_LOGF(LogOpenCVLensCalibration, Error, "OpenCV isn't enabled. Calibration won`t work as expected.");
#endif

	UOpenCVLensCalibrator* Result = NewObject<UOpenCVLensCalibrator>();
	Result->Reset(InBoardWidth, InBoardHeight, InSquareSize, bInUseFisheyeModel);
	return Result;
}

void UOpenCVLensCalibrator::Reset(int32 InBoardWidth, int32 InBoardHeight, float InSquareSize/*=3.f*/, bool bInUseFisheyeModel /*= false*/)
{
	SquareSize = InSquareSize;
	bUseFisheyeModel = bInUseFisheyeModel;
	MinimumCornerCoordinates = FVector2D(FLT_MAX, FLT_MAX);
	MaximumCornerCoordinates = FVector2D(FLT_MIN, FLT_MIN);

#if WITH_OPENCV
	BoardSize = cv::Size(InBoardWidth, InBoardHeight);
	// Assuming the chessboard is at the origin lying flat on the z plane, construct object coordinates for it
	BoardPoints.clear();
	BoardPoints.reserve(BoardSize.height * BoardSize.width);
	for (int i = 0; i < BoardSize.height; ++i)
	{
		for (int j = 0; j < BoardSize.width; ++j)
		{
			BoardPoints.push_back(cv::Point3f(float(j*SquareSize), float(i*SquareSize), 0));
		}
	}

	// Reserve space for a few samples
	ImagePoints.clear();
	ImagePoints.reserve(25);
#endif
}

bool UOpenCVLensCalibrator::FeedRenderTarget(UTextureRenderTarget2D* InTextureRT)
{
#if WITH_OPENCV
	if (InTextureRT == nullptr)
	{
		UE_LOGF(LogOpenCVLensCalibration, Error, "Invalid render target was fed to LensCalibrator");
		return false;
	}

	TArray<uint8> RawData;
	FRenderTarget* RenderTarget = InTextureRT->GameThread_GetRenderTargetResource();
	const EPixelFormat Format = InTextureRT->GetFormat();
	bool bReadSuccess = false;
	switch (Format)
	{
	case PF_FloatRGBA:
	{
		TArray<FFloat16Color> FloatColors;
		bReadSuccess = RenderTarget->ReadFloat16Pixels(FloatColors);
		if (bReadSuccess)
		{
			RawData.AddUninitialized(FloatColors.Num() * 3); // We need to convert float to uint8
			for (int I = 0; I < FloatColors.Num(); I++)
			{
				// OpenCV expects BGR ordering and we're ignoring the alpha component
				RawData[I * 3 + 0] = (uint8)(FloatColors[I].B*255.0f);
				RawData[I * 3 + 1] = (uint8)(FloatColors[I].G*255.0f);
				RawData[I * 3 + 2] = (uint8)(FloatColors[I].R*255.0f);
			}
		}

		break;
	}
	case PF_B8G8R8A8:
	{
		TArray<FColor> Colors;
		bReadSuccess = RenderTarget->ReadPixels(Colors);
		if (bReadSuccess)
		{
			RawData.AddUninitialized(Colors.Num() * 3);
			for (int I = 0; I < Colors.Num(); I++)
			{
				// Strip out the alpha component for OpenCV
				RawData[I * 3 + 0] = Colors[I].B;
				RawData[I * 3 + 1] = Colors[I].G;
				RawData[I * 3 + 2] = Colors[I].R;
			}
		}

		break;
	}
	default:
	{
		UE_LOGF(LogOpenCVLensCalibration, Warning, "Invalid pixel format in render target %ls", *InTextureRT->GetName());
		break;
	}
	}

	if (bReadSuccess == false)
	{
		// Either invalid texture data or unsupported texture format
		return false;
	}

	cv::Mat Image(InTextureRT->SizeY, InTextureRT->SizeX, CV_8UC3, (void*)RawData.GetData());
	return Feed(Image);
#else
	return false;
#endif
}

bool UOpenCVLensCalibrator::FeedImage(const FString& FilePath)
{
#if WITH_OPENCV
	const FTCHARToUTF8 FilePathUtf8(*FilePath);
	const cv::String CVPath(FilePathUtf8.Get(), FilePathUtf8.Length());

	cv::Mat RGBImage = cv::imread(CVPath, cv::IMREAD_UNCHANGED);
	if (RGBImage.empty() == false)
	{
		cv::Mat BGRImage;
		cv::cvtColor(RGBImage, BGRImage, cv::COLOR_RGBA2BGR);
		return Feed(BGRImage);
	}
	else
	{
		return false;
	}
#else
	return false;
#endif
}

#if WITH_OPENCV
bool UOpenCVLensCalibrator::Feed(const cv::Mat& InImage)
{
	//Validate image size before going further
	ImageSize = InImage.size();
	if (ImageSize.empty())
	{
		return false;
	}

	std::vector<cv::Point2f> Corners;
	Corners.reserve(BoardSize.height * BoardSize.width);

	//Using flag CV_CALIB_CB_FAST_CHECK is faster but didn`t catch corners on some images. 
	cv::Mat Gray;
	cv::cvtColor(InImage, Gray, cv::COLOR_BGR2GRAY);
	const bool bFound = cv::findChessboardCorners(Gray, BoardSize, Corners, cv::CALIB_CB_ADAPTIVE_THRESH | cv::CALIB_CB_NORMALIZE_IMAGE);
	if (bFound)
	{
		cv::cornerSubPix(Gray, Corners, cv::Size(11, 11), cv::Size(-1, -1), cv::TermCriteria(cv::TermCriteria::Type::EPS + cv::TermCriteria::Type::COUNT, 30, 0.001));
		ImagePoints.push_back(Corners);

		//Update min/max coordinates to help user cover the whole lens.
		for (const cv::Point2f& Point : Corners)
		{
			MinimumCornerCoordinates.X = FMath::Min(MinimumCornerCoordinates.X, Point.x);
			MinimumCornerCoordinates.Y = FMath::Min(MinimumCornerCoordinates.Y, Point.y);
			MaximumCornerCoordinates.X = FMath::Max(MaximumCornerCoordinates.X, Point.x);
			MaximumCornerCoordinates.Y = FMath::Max(MaximumCornerCoordinates.Y, Point.y);
		}
	}

	return bFound;
}
#endif

bool UOpenCVLensCalibrator::CalculateLensParameters(FOpenCVLensDistortionParameters& OutLensDistortionParameters, float& OutMarginOfError, FOpenCVCameraViewInfo& OutCameraViewInfo)
{
#if WITH_OPENCV

	if (ImagePoints.empty())
	{
		return false;
	}

	cv::Mat DistortionCoefficients;
	cv::Mat CameraMatrix = cv::Mat::eye(3, 3, CV_64F);
	OutMarginOfError = FLT_MAX;
	{
		// Reserve space for Rotation and Translation vectors to compute the total error of our calibration. 
		std::vector<cv::Mat> Rvecs, Tvecs;
		Rvecs.reserve(ImagePoints.size());
		Tvecs.reserve(ImagePoints.size());

		// calibrateCamera requires object points for each image capture, even though they're all the same object
		// (the chessboard) in all cases.
		std::vector<std::vector<cv::Point3f>> ObjectPoints;
		ObjectPoints.resize(ImagePoints.size(), BoardPoints);

		if (bUseFisheyeModel)
		{
			//fisheye calibration doesn't like it with 1 image
			if (ImagePoints.size() > 1)
			{
				OutMarginOfError = (float)cv::fisheye::calibrate(ObjectPoints, ImagePoints, ImageSize, CameraMatrix, DistortionCoefficients, Rvecs, Tvecs, cv::fisheye::CALIB_RECOMPUTE_EXTRINSIC + cv::fisheye::CALIB_FIX_SKEW, cv::TermCriteria(cv::TermCriteria::Type::EPS + cv::TermCriteria::Type::COUNT, 30, 1e-6));
			}
			else
			{
				UE_LOGF(LogOpenCVLensCalibration, Warning, "Fisheye calibration requires at least 2 samples.");
				return false;
			}
		}
		else
		{
			OutMarginOfError = (float)cv::calibrateCamera(ObjectPoints, ImagePoints, ImageSize, CameraMatrix, DistortionCoefficients, Rvecs, Tvecs);
		}
	}

	if (!checkRange(CameraMatrix) || !checkRange(DistortionCoefficients))
	{
		return false;
	}

	// Convert the params to a Unreal Editor struct
	{
		//Fisheye camera model vs pinhole camera model differs slightly in parameter assignment
		//Pinhole can have up to 6 radial distortion parameters and 2 tangential parameters
		//where fisheye model only have 4 'k' parameters.
		if (bUseFisheyeModel)
		{
			check(DistortionCoefficients.rows >= 1);
			OutLensDistortionParameters.K1 = DistortionCoefficients.at<double>(0);
			OutLensDistortionParameters.K2 = DistortionCoefficients.at<double>(1);
			OutLensDistortionParameters.K3 = DistortionCoefficients.at<double>(2);
			OutLensDistortionParameters.K4 = DistortionCoefficients.at<double>(3);
		}
		else
		{
			// The DistortionCoefficients matrix is a one row matrix for pinhole model
			check(DistortionCoefficients.rows == 1);
			OutLensDistortionParameters.K1 = DistortionCoefficients.at<double>(0);
			OutLensDistortionParameters.K2 = DistortionCoefficients.at<double>(1);
			OutLensDistortionParameters.P1 = DistortionCoefficients.at<double>(2);
			OutLensDistortionParameters.P2 = DistortionCoefficients.at<double>(3);

			// The docs make it sound like third (and fourth and fifth) radial coefficients may be optional
			OutLensDistortionParameters.K3 = DistortionCoefficients.cols >= 5 ? DistortionCoefficients.at<double>(4) : .0f;
			OutLensDistortionParameters.K4 = DistortionCoefficients.cols >= 6 ? DistortionCoefficients.at<double>(5) : .0f;
			OutLensDistortionParameters.K5 = DistortionCoefficients.cols >= 7 ? DistortionCoefficients.at<double>(6) : .0f;
			OutLensDistortionParameters.K6 = DistortionCoefficients.cols >= 8 ? DistortionCoefficients.at<double>(7) : .0f;
		}

		//Save camera matrix with normalized values
		check(CameraMatrix.rows == 3 && CameraMatrix.cols == 3);
		OutLensDistortionParameters.F.X = CameraMatrix.at<double>(0, 0) / ImageSize.width;
		OutLensDistortionParameters.F.Y = CameraMatrix.at<double>(1, 1) / ImageSize.height;
		OutLensDistortionParameters.C.X = CameraMatrix.at<double>(0, 2) / ImageSize.width;
		OutLensDistortionParameters.C.Y = CameraMatrix.at<double>(1, 2) / ImageSize.height;

		OutLensDistortionParameters.bUseFisheyeModel = bUseFisheyeModel;
	}

	// Estimate camera view information (FOV, Aspect ratio...)
	{
		double FocalLengthRatio, FovX, FovY, FocalLength_Unused;
		cv::Point2d PrincipalPoint_Unused;
		// We pass in zero aperture size as it is unknown. (It is only required for calculating focal length and the principal point)
		cv::calibrationMatrixValues(CameraMatrix, ImageSize, 0.0, 0.0, FovX, FovY, FocalLength_Unused, PrincipalPoint_Unused, FocalLengthRatio);
		OutCameraViewInfo.HorizontalFOV = FovX;
		OutCameraViewInfo.VerticalFOV = FovY;
		OutCameraViewInfo.FocalLengthRatio = FocalLengthRatio;
	}

	return true;
#else
	return false;
#endif
}
