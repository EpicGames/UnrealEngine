// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


// IWYU pragma: begin_keep
#include "OpenCVLensDistortionParameters.h"

#if WITH_OPENCV
#include "OpenCVHelper.h"

#include "PreOpenCVHeaders.h"
#include "opencv2/imgcodecs.hpp"
#include "PostOpenCVHeaders.h"
#endif
// IWYU pragma: end_keep

#include "OpenCVLensCalibrator.generated.h"


class UTextureRenderTarget2D;


UCLASS(meta = (BlueprintSpawnableComponent))
class UOpenCVLensCalibrator : public UObject
{
	GENERATED_BODY()

private:
	/**
	 * Default constructor for an OpenCV lens calibration object.
	 */
	UOpenCVLensCalibrator(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
		: UObject(ObjectInitializer)
	{}
public:

	/**
	 * @param BoardWidth The width of the checkerboard used to calibrate the camera counted as number of inner edges.
	 * @param BoardHeight The height of the checkerboard used to calibrate the camera counted as number of inner edges.
	 * @param SquareSize The width of each square in (potentially arbitrary) world units.
	 * @param bUseFisheyeModel Specifies if the calibrator is to use the fisheye camera model from OpenCV
	 */
	UFUNCTION(BlueprintCallable, Category = "LensDistortion|OpenCV|Calibration")
	static UOpenCVLensCalibrator* CreateCalibrator(int32 BoardWidth = 7, int32 BoardHeight = 5, float SquareSize = 3.f, bool bUseFisheyeModel = false);

	/**
	 * @param InBoardWidth The width of the checkerboard used to calibrate the camera counted as number of inner edges.
	 * @param InBoardHeight The height of the checkerboard used to calibrate the camera counted as number of inner edges.
	 * @param InSquareSize The width of each square in (potentially arbitrary) world units.
	 * @param bInUseFisheyeModel Specifies if the calibrator is to use the fisheye camera model from OpenCV
	 */
	void Reset(int32 InBoardWidth, int32 InBoardHeight, float InSquareSize = 3.f, bool bInUseFisheyeModel = false);

	/**
	 * Feeds a render target to the calibration. It must contain a checkerboard somewhere in the image.
	 * The images fed in should come from the same camera.
	 * @return true if the calibrator found a checkerboard in the image.
	 */
	UFUNCTION(BlueprintCallable, Category = "LensDistortion|OpenCV|Calibration")
	bool FeedRenderTarget(UTextureRenderTarget2D* TextureRenderTarget);

	/**
	 * Feeds an image to the calibration. It must contain a checkerboard somewhere in the image.
	 * The images fed in should come from the same camera.
	 * @return true if the calibrator found a checkerboard in the image.
	 */
	UFUNCTION(BlueprintCallable, Category = "LensDistortion|OpenCV|Calibration")
	bool FeedImage(const FString& FilePath);

	/**
	 * @param LensDistortionParameters the calculated distortion data from the images passed into the calibrator.
	 * @param MarginOfError returned reprojection error of the calibration
	 * @param CameraViewInfo returned information about the camera view obtained from calibration parameters
	 * @return true if there was enough data to calculate the distortion
	 */
	UFUNCTION(BlueprintCallable, Category = "LensDistortion|OpenCV|Calibration")
	bool CalculateLensParameters(FOpenCVLensDistortionParameters& LensDistortionParameters, float& MarginOfError, FOpenCVCameraViewInfo& CameraViewInfo);

private:

	/**
	 * @param InImage The input image in matrix form formatted as BGR.
	 * @return true if checkerboard corners were found.
	 */
#if WITH_OPENCV
	bool Feed(const cv::Mat& InImage);
#endif

public:
	/** Smallest coordinates of a grid corner that was found. For best result, try to cover full resolution of the input. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Calibration")
	FVector2D MinimumCornerCoordinates;

	/** Biggest coordinates of a grid corner that was found. For best result, try to cover full resolution of the input. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Calibration")
	FVector2D MaximumCornerCoordinates;

private:

#if WITH_OPENCV
	/** Size of a square in the grid. Potentially useless. */
	std::vector<std::vector<cv::Point2f>> ImagePoints;

	/** Size of a square in the grid. Potentially useless. */
	std::vector<cv::Point3f> BoardPoints;

	/** Size of input image used for calibration in pixels. */
	cv::Size ImageSize;

	/** Size of the checkerboard in number of squares. */
	cv::Size BoardSize;
#endif

	/** Size of a square in the grid. Potentially useless. */
	float SquareSize;

	/** Specifies if Fisheye camera model is to be used. */
	bool bUseFisheyeModel;
};
