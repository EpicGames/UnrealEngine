// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class OpenCVLensDistortion : ModuleRules
	{
		public OpenCVLensDistortion(ReadOnlyTargetRules Target) : base(Target)
		{            
            PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"OpenCVHelper",
					"OpenCV",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"Projects",
					"RenderCore",
					"RHI",
					"Projects",
				}
			);
		}
	}
}
