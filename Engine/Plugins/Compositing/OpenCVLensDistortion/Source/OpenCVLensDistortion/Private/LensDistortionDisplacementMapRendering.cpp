// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenCVLensDistortionParameters.h"

#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "GlobalShader.h"
#include "IOpenCVLensDistortionModule.h"
#include "PipelineStateCache.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RHIStaticStates.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneInterface.h"
#include "ShaderParameterUtils.h"
#include "TextureResource.h"
#include "RenderingThread.h"


//Parameters for the grid we'll use to get the reciprocal of our undistortion map
static const uint32 kGridSubdivisionX = 32;
static const uint32 kGridSubdivisionY = 16;

#if WITH_OPENCV
class FLensDistortionDisplacementMapGenerationShader : public FGlobalShader
{
	DECLARE_INLINE_TYPE_LAYOUT(FLensDistortionDisplacementMapGenerationShader, NonVirtual);
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("GRID_SUBDIVISION_X"), kGridSubdivisionX);
		OutEnvironment.SetDefine(TEXT("GRID_SUBDIVISION_Y"), kGridSubdivisionY);
	}

	FLensDistortionDisplacementMapGenerationShader() {}

	FLensDistortionDisplacementMapGenerationShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		PixelUVSize.Bind(Initializer.ParameterMap, TEXT("PixelUVSize"));
		UndistortDisplacementMap.Bind(Initializer.ParameterMap, TEXT("UndistortDisplacementMap"));
		BilinearSampler.Bind(Initializer.ParameterMap, TEXT("BilinearClampedSampler"));
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTextureResource* PreComputedDisplacementMap, const FIntPoint& DisplacementMapResolution)
	{
		FVector2f PixelUVSizeValue(1.f / float(DisplacementMapResolution.X), 1.f / float(DisplacementMapResolution.Y));

		SetShaderValue(BatchedParameters, PixelUVSize, PixelUVSizeValue);
		SetTextureParameter(BatchedParameters, UndistortDisplacementMap, BilinearSampler, TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI(), PreComputedDisplacementMap->TextureRHI);
	}

private:
	
	LAYOUT_FIELD(FShaderParameter, PixelUVSize);
	LAYOUT_FIELD(FShaderResourceParameter, UndistortDisplacementMap);
	LAYOUT_FIELD(FShaderResourceParameter, BilinearSampler);
};


class FLensDistortionDisplacementMapGenerationVS : public FLensDistortionDisplacementMapGenerationShader
{
	DECLARE_SHADER_TYPE(FLensDistortionDisplacementMapGenerationVS, Global);

public:

	/** Default constructor. */
	FLensDistortionDisplacementMapGenerationVS() {}

	/** Initialization constructor. */
	FLensDistortionDisplacementMapGenerationVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FLensDistortionDisplacementMapGenerationShader(Initializer)
	{
	}
};


class FLensDistortionDisplacementMapGenerationPS : public FLensDistortionDisplacementMapGenerationShader
{
	DECLARE_SHADER_TYPE(FLensDistortionDisplacementMapGenerationPS, Global);

public:

	/** Default constructor. */
	FLensDistortionDisplacementMapGenerationPS() {}

	/** Initialization constructor. */
	FLensDistortionDisplacementMapGenerationPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FLensDistortionDisplacementMapGenerationShader(Initializer)
	{ }
};


IMPLEMENT_SHADER_TYPE(, FLensDistortionDisplacementMapGenerationVS, TEXT("/Plugin/OpenCVLensDistortion/Private/DisplacementMapGeneration.usf"), TEXT("MainVS"), SF_Vertex)
IMPLEMENT_SHADER_TYPE(, FLensDistortionDisplacementMapGenerationPS, TEXT("/Plugin/OpenCVLensDistortion/Private/DisplacementMapGeneration.usf"), TEXT("MainPS"), SF_Pixel)
#endif

static void DrawUVDisplacementToRenderTargetFromPreComputedDisplacementMap_RenderThread(FRHICommandListImmediate& RHICmdList
																			, const FTextureResource* PreComputedDisplacementMap
																			, const FName& TextureRenderTargetName
																			, FTextureRenderTargetResource* OutTextureRenderTargetResource
																			, ERHIFeatureLevel::Type FeatureLevel)
{
#if WITH_OPENCV
	check(IsInRenderingThread());
	check(PreComputedDisplacementMap);

	SCOPED_DRAW_EVENTF(RHICmdList, DrawUVDisplacementToRenderTargetFromPreComputedDisplacementMap, TEXT("OpenCVLensDistortionDisplacementMapGeneration %s"), TextureRenderTargetName);

	// Set render target.
	FRHIRenderPassInfo RPInfo(OutTextureRenderTargetResource->GetRenderTargetTexture(), ERenderTargetActions::Clear_Store);
	RHICmdList.BeginRenderPass(RPInfo, TEXT("DrawUVDisplacementFromPrecomputedDisplacementMap"));
	{

		// Update viewport.
		const FIntPoint DisplacementMapResolution(OutTextureRenderTargetResource->GetSizeX(), OutTextureRenderTargetResource->GetSizeY());
		RHICmdList.SetViewport(0, 0, 0.f, DisplacementMapResolution.X, DisplacementMapResolution.Y, 1.f);

		// Get shaders.
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(FeatureLevel);
		TShaderMapRef< FLensDistortionDisplacementMapGenerationVS > VertexShader(GlobalShaderMap);
		TShaderMapRef< FLensDistortionDisplacementMapGenerationPS > PixelShader(GlobalShaderMap);

		// Set the graphic pipeline state.
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		// Update shader uniform parameters.
		SetShaderParametersLegacyVS(RHICmdList, VertexShader, PreComputedDisplacementMap, DisplacementMapResolution);
		SetShaderParametersLegacyPS(RHICmdList, PixelShader, PreComputedDisplacementMap, DisplacementMapResolution);

		// Draw grid.
		const uint32 PrimitiveCount = kGridSubdivisionX * kGridSubdivisionY * 2;
		RHICmdList.DrawPrimitive(0, PrimitiveCount, 1);
	}
	RHICmdList.EndRenderPass();
	TransitionAndCopyTexture(RHICmdList, OutTextureRenderTargetResource->GetRenderTargetTexture(), OutTextureRenderTargetResource->TextureRHI, {});
#endif
}

void FOpenCVLensDistortionParameters::DrawDisplacementMapToRenderTarget(UWorld* InWorld, UTextureRenderTarget2D* InOutputRenderTarget, UTexture2D* InPreComputedUndistortDisplacementMap)
{
#if WITH_OPENCV
	check(IsInGameThread());

	if (!InOutputRenderTarget)
	{
		UE_LOGF(LogOpenCVLensDistortion, Error, "Invalid render target to draw on.");
		return;
	}

	if(InPreComputedUndistortDisplacementMap == nullptr || InPreComputedUndistortDisplacementMap->GetResource() == nullptr)
	{
		UE_LOGF(LogOpenCVLensDistortion, Error, "Precomputed displacement map is required to generate final displacement maps.");
		return;
	}

	//Prepare parameters for render command
	const FName TextureRenderTargetName = InOutputRenderTarget->GetFName();
	FTextureRenderTargetResource* TextureRenderTargetResource = InOutputRenderTarget->GameThread_GetRenderTargetResource();
	const FTextureResource* PreComputedMapResource = InPreComputedUndistortDisplacementMap->GetResource();
	ERHIFeatureLevel::Type FeatureLevel = InWorld->Scene->GetFeatureLevel();

	ENQUEUE_RENDER_COMMAND(CaptureCommand)
	(
		[PreComputedMapResource, TextureRenderTargetResource, TextureRenderTargetName, FeatureLevel](FRHICommandListImmediate& RHICmdList)
		{
			DrawUVDisplacementToRenderTargetFromPreComputedDisplacementMap_RenderThread(RHICmdList, PreComputedMapResource, TextureRenderTargetName,	TextureRenderTargetResource, FeatureLevel);
		}
	);
#endif
}
