// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenCVLensDistortionBlueprintLibrary.h"


UOpenCVLensDistortionBlueprintLibrary::UOpenCVLensDistortionBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{ }


void UOpenCVLensDistortionBlueprintLibrary::DrawDisplacementMapToRenderTarget(const UObject* WorldContextObject
																			, UTextureRenderTarget2D* OutputRenderTarget
																			, UTexture2D* PreComputedUndistortDisplacementMap)
{
	FOpenCVLensDistortionParameters::DrawDisplacementMapToRenderTarget(WorldContextObject->GetWorld(), OutputRenderTarget, PreComputedUndistortDisplacementMap);
}

UTexture2D* UOpenCVLensDistortionBlueprintLibrary::CreateUndistortUVDisplacementMap(const FOpenCVLensDistortionParameters& LensParameters, const FIntPoint& ImageSize, const float CroppingFactor, FOpenCVCameraViewInfo& CameraViewInfo)
{
	return LensParameters.CreateUndistortUVDisplacementMap(ImageSize, CroppingFactor, CameraViewInfo);
}
