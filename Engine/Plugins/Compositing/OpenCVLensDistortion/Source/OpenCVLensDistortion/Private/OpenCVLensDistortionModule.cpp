// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOpenCVLensDistortionModule.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Interfaces/IPluginManager.h"
#include "ShaderCore.h"


DEFINE_LOG_CATEGORY(LogOpenCVLensDistortion)

//////////////////////////////////////////////////////////////////////////
// FOpenCVLensDistortionModule
class FOpenCVLensDistortionModule : public IOpenCVLensDistortionModule
{
public:
	virtual void StartupModule() override
	{
		// Maps virtual shader source directory /Plugin/OpenCVLensDistortion to the plugin's actual Shaders directory.
		FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("OpenCVLensDistortion"))->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(TEXT("/Plugin/OpenCVLensDistortion"), PluginShaderDir);
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FOpenCVLensDistortionModule, OpenCVLensDistortion);


void IOpenCVLensDistortionModule::StartupModule()
{
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("OpenCVLensDistortion"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/OpenCVLensDistortion"), PluginShaderDir);
}
