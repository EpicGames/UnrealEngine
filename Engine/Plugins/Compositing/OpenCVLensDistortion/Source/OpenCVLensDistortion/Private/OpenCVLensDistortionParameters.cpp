// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenCVLensDistortionParameters.h"

#include "Engine/Texture2D.h"
#include "TextureResource.h"

#if WITH_OPENCV
// IWYU pragma: begin_keep
#include "PreOpenCVHeaders.h"
#include "opencv2/calib3d.hpp"
#include "opencv2/imgproc.hpp"
#include "PostOpenCVHeaders.h"
// IWYU pragma: end_keep
#else
#include "IOpenCVLensDistortionModule.h"
#endif


UTexture2D* FOpenCVLensDistortionParameters::CreateUndistortUVDisplacementMap(const FIntPoint& InImageSize, const float InCroppingFactor, FOpenCVCameraViewInfo& OutCameraViewInfo) const
{
#if WITH_OPENCV
	cv::Mat GeneratedMap1(InImageSize.X, InImageSize.Y, CV_32FC1);
	cv::Mat GeneratedMap2(InImageSize.X, InImageSize.Y, CV_32FC1);

	// Use OpenCV to generate the initial direct UVmap
	{
		cv::Size ImageSizeCV(InImageSize.X, InImageSize.Y);
		cv::Mat Identity = cv::Mat::eye(3, 3, CV_64F);

		cv::Mat CameraMatrixCV = CreateOpenCVCameraMatrix(FVector2D(InImageSize.X, InImageSize.Y));
		cv::Mat DistortionCoefficientsCV = ConvertToOpenCVDistortionCoefficients();

		// Calculate a new camera matrix based on the camera distortion coefficients and the desired cropping factor.
		//Compute the direct UVMap based on this new camera matrix.
		cv::Mat NewCameraMatrix = cv::Mat::eye(3, 3, CV_64F);
		if (bUseFisheyeModel)
		{
			cv::fisheye::estimateNewCameraMatrixForUndistortRectify(CameraMatrixCV, DistortionCoefficientsCV, ImageSizeCV, Identity, NewCameraMatrix, 1.0f - InCroppingFactor);
			cv::fisheye::initUndistortRectifyMap(CameraMatrixCV, DistortionCoefficientsCV, Identity, NewCameraMatrix, ImageSizeCV, GeneratedMap1.type(), GeneratedMap1, GeneratedMap2);
		}
		else
		{
			NewCameraMatrix = getOptimalNewCameraMatrix(CameraMatrixCV, DistortionCoefficientsCV, ImageSizeCV, 1.0f - InCroppingFactor);
			cv::initUndistortRectifyMap(CameraMatrixCV, DistortionCoefficientsCV, Identity, NewCameraMatrix, ImageSizeCV, GeneratedMap1.type(), GeneratedMap1, GeneratedMap2);
		}
	
		// Estimate field of view of the undistorted image
		double FocalLengthRatio, FovX, FovY, FocalLength_Unused;
		cv::Point2d PrincipalPoint_Unused;

		// We pass in zero aperture size as it is unknown. (It is only required for calculating focal length and the principal point)
		calibrationMatrixValues(NewCameraMatrix, ImageSizeCV, 0.0, 0.0, FovX, FovY, FocalLength_Unused, PrincipalPoint_Unused, FocalLengthRatio);
		OutCameraViewInfo.HorizontalFOV = FovX;
		OutCameraViewInfo.VerticalFOV = FovY;
		OutCameraViewInfo.FocalLengthRatio = FocalLengthRatio;
	}

	// Now convert the raw map arrays to an unreal displacement map texture
	UTexture2D* Result = UTexture2D::CreateTransient(InImageSize.X, InImageSize.Y, PF_G16R16F);

	// Lock the texture so it can be modified
	FTexture2DMipMap& Mip = Result->GetPlatformData()->Mips[0];
	uint16* MipData = reinterpret_cast<uint16*>(Mip.BulkData.Lock(LOCK_READ_WRITE));
	check(MipData);

	// Go through each pixel and change to normalized displacement value
	// OpenCV doesn't use half pixel shift coordinate but changing to displacement map fixes it.
	for (int32 Y = 0; Y < InImageSize.Y; Y++)
	{
		uint16* Row = &MipData[Y * InImageSize.X * 2];
		for (int32 X = 0; X < InImageSize.X; X++)
		{
			float UOffset = GeneratedMap1.at<float>(Y, X);
			UOffset -= X;
			UOffset /= static_cast<float>(InImageSize.X);

			float VOffset = GeneratedMap2.at<float>(Y, X);
			VOffset -= Y;
			VOffset /= static_cast<float>(InImageSize.Y);

			// red channel:
			Row[X * 2 + 0] = FFloat16(UOffset).Encoded;
			// green channel:
			Row[X * 2 + 1] = FFloat16(VOffset).Encoded;
		}
	}

	// Unlock the texture data
	Mip.BulkData.Unlock();
	Result->UpdateResource();

	return Result;
#else
	UE_LOGF(LogOpenCVLensDistortion, Error, "Can't create undistortion displacement Map. OpenCV isn't enabled.");
	return nullptr;
#endif
}

