// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "OpenCVLensDistortionParameters.h"
#include "UObject/ObjectMacros.h"

#include "OpenCVLensDistortionBlueprintLibrary.generated.h"


class UTexture2D;
class UTextureRenderTarget2D;


UCLASS(MinimalAPI, meta=(ScriptName="OpenCVLensDistortionLibrary"))
class UOpenCVLensDistortionBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()


	/** Draws UV displacement map within the output render target.
	 * - Red & green channels hold the distort to undistort displacement;
	 * - Blue & alpha channels hold the undistort to distort displacement.
	 * @param WorldContextObject Current world to get the rendering settings from (such as feature level).
	 * @param OutputRenderTarget The render target to draw to. Don't necessarily need to have same resolution or aspect ratio as distorted render.
	 * @param PreComputedUndistortDisplacementMap Distort to undistort displacement pre computed using OpenCV in engine or externally.
	 */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion | OpenCV", meta = (WorldContext = "WorldContextObject"))
	static void DrawDisplacementMapToRenderTarget(const UObject* WorldContextObject, UTextureRenderTarget2D* OutputRenderTarget, UTexture2D* PreComputedUndistortDisplacementMap);

	/**
	 * Creates a texture containing a DisplacementMap in the Red and the Green channel for undistorting a camera image.
	 * This call can take quite some time to process depending on the resolution.
	 * @param LensParameters The Lens distortion parameters with which to compute the UV displacement map.
	 * @param ImageSize The size of the camera image to be undistorted in pixels. Scaled down resolution will have an impact. 
	 * @param CroppingFactor One means OpenCV will attempt to crop out all empty pixels resulting from the process (essentially zooming the image). Zero will keep all pixels.
	 * @param CameraViewInfo Information computed by OpenCV about the undistorted space. Can be used with SceneCapture to adjust FOV.
	 * @return Texture2D containing the distort to undistort space displacement map.
	 */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion | OpenCV", meta = (WorldContext = "WorldContextObject"))
	static UTexture2D* CreateUndistortUVDisplacementMap(const FOpenCVLensDistortionParameters& LensParameters, const FIntPoint& ImageSize, const float CroppingFactor, FOpenCVCameraViewInfo& CameraViewInfo);

	/* Returns true if A is equal to B (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "Equal (LensDistortionParameters)", CompactNodeTitle = "==", Keywords = "== equal", ScriptOperator = "=="), Category = "Lens Distortion")
	static bool EqualEqual_CompareLensDistortionModels(
		const FOpenCVLensDistortionParameters& A,
		const FOpenCVLensDistortionParameters& B)
	{
		return A == B;
	}

	/* Returns true if A is not equal to B (A != B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "NotEqual (LensDistortionParameters)", CompactNodeTitle = "!=", Keywords = "!= not equal", ScriptOperator = "!="), Category = "Lens Distortion")
	static bool NotEqual_CompareLensDistortionModels(
		const FOpenCVLensDistortionParameters& A,
		const FOpenCVLensDistortionParameters& B)
	{
		return A != B;
	}
};
