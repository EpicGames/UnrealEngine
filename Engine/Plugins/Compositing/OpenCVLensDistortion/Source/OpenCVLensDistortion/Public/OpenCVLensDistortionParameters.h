// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Keep OpenCVHelper.h outside of the WITH_OPENCV define so that FOpenCVLensDistortionParametersBase is included
// OpenCVHelper.h has its own WITH_OPENCV guards so having it guarded again here isn't necessary
#include "OpenCVHelper.h"

#include "OpenCVLensDistortionParameters.generated.h"


class UTexture2D;
class UTextureRenderTarget2D;


USTRUCT(BlueprintType)
struct OPENCVLENSDISTORTION_API FOpenCVCameraViewInfo
{
	GENERATED_USTRUCT_BODY()

	FOpenCVCameraViewInfo()
		: HorizontalFOV(0.0f)
		, VerticalFOV(0.0f)
		, FocalLengthRatio(0.0f)
	{ }

	/** Horizontal Field Of View in degrees */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Info")
	float HorizontalFOV;

	/** Vertical Field Of View in degrees */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Info")
	float VerticalFOV;

	/** Focal length aspect ratio -> Fy / Fx */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Info")
	float FocalLengthRatio;
};

/**
 * Mathematic camera model for lens distortion/undistortion.
 * Camera matrix =
 *  | F.X  0  C.x |
 *  |  0  F.Y C.Y |
 *  |  0   0   1  |
 * where F and C are normalized.
 */
USTRUCT(BlueprintType)
struct OPENCVLENSDISTORTION_API FOpenCVLensDistortionParameters : public FOpenCVLensDistortionParametersBase
{
	GENERATED_USTRUCT_BODY()

public:
	/** Draws UV displacement map within the output render target.
	 * - Red & green channels hold the distort to undistort displacement;
	 * - Blue & alpha channels hold the undistort to distort displacement.
	 * @param InWorld Current world to get the rendering settings from (such as feature level).
	 * @param InOutputRenderTarget The render target to draw to. Don't necessarily need to have same resolution or aspect ratio as distorted render.
	 * @param InPreComputedUndistortDisplacementMap Distort to undistort displacement pre computed using OpenCV in engine or externally.
	 */
	static void DrawDisplacementMapToRenderTarget(UWorld* InWorld, UTextureRenderTarget2D* InOutputRenderTarget, UTexture2D* InPreComputedUndistortDisplacementMap);

	/**
	 * Creates a texture containing a DisplacementMap in the Red and the Green channel for undistorting a camera image.
	 * This call can take quite some time to process depending on the resolution.
	 * @param InImageSize The size of the camera image to be undistorted in pixels. Scaled down resolution will have an impact.
	 * @param InCroppingFactor One means OpenCV will attempt to crop out all empty pixels resulting from the process (essentially zooming the image). Zero will keep all pixels.
	 * @param OutCameraViewInfo Information computed by OpenCV about the undistorted space. Can be used with SceneCapture to adjust FOV.
	 */
	UTexture2D* CreateUndistortUVDisplacementMap(const FIntPoint& InImageSize, const float InCroppingFactor, FOpenCVCameraViewInfo& OutCameraViewInfo) const;
};
